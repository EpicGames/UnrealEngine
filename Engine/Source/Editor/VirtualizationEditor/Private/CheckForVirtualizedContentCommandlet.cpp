// Copyright Epic Games, Inc. All Rights Reserved.

#include "CheckForVirtualizedContentCommandlet.h"

#include "CommandletUtils.h"
#include "HAL/FileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "UObject/PackageTrailer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CheckForVirtualizedContentCommandlet)

UCheckForVirtualizedContentCommandlet::UCheckForVirtualizedContentCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UCheckForVirtualizedContentCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UCheckForVirtualizedContentCommandlet);

	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	bool bNoVAContentInEngine = false;
	bool bNoVAContentInProject = false;

	TArray<FString> DirectoriesToCheck;

	bool bInputProvided = false;

	for (const FString& Switch : Switches)
	{
		FString InputPath;

		if (Switch == TEXT("CheckEngine"))
		{
			bNoVAContentInEngine = true;
			bInputProvided = true;
		}
		else if (Switch == TEXT("CheckProject"))
		{
			bNoVAContentInProject = true;
			bInputProvided = true;
		}
		else if (FParse::Value(*Switch, TEXT("CheckDir="), InputPath))
		{
			InputPath.ParseIntoArray(DirectoriesToCheck, TEXT("+"), true);
			bInputProvided = true;
		}
		else if (Switch == TEXT("OutputPaths"))
		{
			bOutputPackageNames = false;
		}
	}

	if (bInputProvided == false)
	{
		UE_LOGF(LogVirtualization, Error, "No input was provided for the commandlet. Use '-CheckEngine', '-CheckProject' or '-CheckDir=...'");
		return 2;
	}

	TArray<FString> EnginePackages;
	TArray<FString> ProjectPackages;

	if (bNoVAContentInEngine)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindEnginePackages);

		// When checking engine content it is assumed that we want to check ALL engine content not just the plugins that
		// happen to be enabled for the current project. This is why we cannot use the asset registry by calling
		// UE::Virtualization::FindPackages as that will only find th engine content enabled for the current project.
		// Instead we search the engine from it's root directory.
		EnginePackages =  UE::Virtualization::FindPackagesInDirectory(FPaths::EngineDir());
	}

	if (bNoVAContentInProject)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindProjectPackages);

		ProjectPackages = UE::Virtualization::FindPackages(UE::Virtualization::EFindPackageFlags::ExcludeEngineContent);
	}

	bool bAllContentValid = true;

	if (bNoVAContentInEngine)
	{
		if (!TryValidateContent(TEXT("Engine"), EnginePackages))
		{
			bAllContentValid = false;
		}
	}

	if (bNoVAContentInProject)
	{
		if (!TryValidateContent(TEXT("Project"), ProjectPackages))
		{
			bAllContentValid = false;
		}
	}

	if (!DirectoriesToCheck.IsEmpty())
	{
		for (const FString& Directory : DirectoriesToCheck)
		{
			if (!TryValidateDirectory(Directory))
			{
				bAllContentValid = false;
			}
		}
	}

	UE_LOGF(LogVirtualization, Display, "********************************************************************************");

	return bAllContentValid ? 0 : 1;
}

TArray<FString> UCheckForVirtualizedContentCommandlet::FindVirtualizedPackages(const TArray<FString>& PackagePaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParsePackageTrailers);

	TArray<FString> VirtualizedPackages;

	for (const FString& Path : PackagePaths)
	{
		UE::FPackageTrailer Trailer;
		if (UE::FPackageTrailer::TryLoadFromFile(Path, Trailer))
		{
			const int32 NumVirtualizedPayloads = Trailer.GetNumPayloads(UE::EPayloadStorageType::Virtualized);
			if (NumVirtualizedPayloads > 0)
			{
				FString PackageName;
				if (bOutputPackageNames && FPackageName::TryConvertFilenameToLongPackageName(Path, PackageName))
				{
					VirtualizedPackages.Emplace(MoveTemp(PackageName));
				}
				else
				{
					if (FPaths::IsRelative(Path))
					{
						VirtualizedPackages.Add(FPaths::ConvertRelativePathToFull(Path));
					}
					else
					{
						VirtualizedPackages.Add(Path);
					}
				}

			}
		}
	}

	return VirtualizedPackages;
}

bool UCheckForVirtualizedContentCommandlet::TryValidateContent(const TCHAR* DebugName, const TArray<FString>& Packages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*WriteToString<128>("TryValidateContent - ", DebugName));

	check(DebugName != nullptr);

	UE_LOGF(LogVirtualization, Display, "********************************************************************************");
	UE_LOGF(LogVirtualization, Display, "Looking for virtualized payloads in %ls content...", DebugName);
	UE_LOGF(LogVirtualization, Display, "Found %d %ls package(s)", Packages.Num(), DebugName);

	TArray<FString> VirtualizedPackages = FindVirtualizedPackages(Packages);

	if (VirtualizedPackages.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Display, "No virtualized packages were found in %ls content", DebugName);
		return true;
	}
	else
	{
		for (FString& Path : VirtualizedPackages)
		{
			UE_LOGFMT(LogVirtualization, Error, "Package {PackagePath} contains virtualized payloads", Path);
		}

		UE_LOGF(LogVirtualization, Error, "Found %d virtualized package(s) in %ls content", VirtualizedPackages.Num(), DebugName);
		return false;
	}
}

bool UCheckForVirtualizedContentCommandlet::TryValidateDirectory(const FString& Directory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TryValidateDirectory);

	UE_LOGF(LogVirtualization, Display, "********************************************************************************");
	UE_LOGF(LogVirtualization, Display, "Searching directory '%ls' for virtualized packages...", *Directory);

	if (!IFileManager::Get().DirectoryExists(*Directory))
	{
		UE_LOGF(LogVirtualization, Error, "Directory '%ls' could not be found!", *Directory);
		return false;
	}

	TArray<FString> DirectoryPackages;
	DirectoryPackages = UE::Virtualization::FindPackagesInDirectory(Directory);

	if (DirectoryPackages.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Display, "Found no packages under '%ls'", *Directory);
		return true;
	}

	UE_LOGF(LogVirtualization, Display, "Found %d package(s) under '%ls'", DirectoryPackages.Num(), *Directory);
	UE_LOGF(LogVirtualization, Display, "Looking for virtualized payloads under directory...");

	TArray<FString> VirtualizedPackages = FindVirtualizedPackages(DirectoryPackages);

	if (VirtualizedPackages.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Display, "No virtualized packages were found under '%ls'", *Directory);
	}
	else
	{
		for (FString& Path : VirtualizedPackages)
		{
			UE_LOGFMT(LogVirtualization, Error, "Package {PackagePath} contains virtualized payloads", Path);
		}

		UE_LOGF(LogVirtualization, Error, "Found %d virtualized package(s) under '%ls'", VirtualizedPackages.Num(), *Directory);
	}

	return true;
}
