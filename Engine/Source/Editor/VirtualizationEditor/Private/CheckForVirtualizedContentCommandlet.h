// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "UObject/ObjectMacros.h"

#include "CheckForVirtualizedContentCommandlet.generated.h"

/**
 * Used to validate that content does not contain virtualized payloads. The current
 * cmdline options are:
 * -CheckEngine
 *     Checks all packages currently mounted in the engine
 * -CheckProject
 *     Checks all packages currently mounted in the current project
 * -CheckDir=XYZ (XYZ is the path to the directory, use '+' as the delimiter if supplying
 * more than one path)
 *     Checks packages in the given directory and all subdirectories
 * any or all of which can be passed to the commandlet.
 * -OutputPaths
 *     Report virtualized packages by path rather than package name.
 *
 * If virtualized payloads are found then the package path (or file path if CheckDir
 * indicates a directory that is not currently mounted by the project) is logged
 * as an error and the commandlet will eventually return 1 as a failure value.
 *
 * Because the commandlet is the VirtualizationEditor module it needs to be invoked
 * with the command line:
 * -run="VirtualizationEditor.CheckForVirtualizedContent"
 *
 * Followed by
 */
UCLASS()
class UCheckForVirtualizedContentCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);

private:

	TArray<FString> FindVirtualizedPackages(const TArray<FString>& PackagePaths);

	bool TryValidateContent(const TCHAR* DebugName, const TArray<FString>& Packages);
	bool TryValidateDirectory(const FString& Directory);

	bool bOutputPackageNames = true;

};
