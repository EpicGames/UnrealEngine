// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommandletUtils.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/ParallelFor.h"
#include "Containers/Set.h"
#include "IO/IoHash.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "String/ParseTokens.h"
#include "Tasks/Task.h"	
#include "UObject/PackageTrailer.h"
#include "Virtualization/VirtualizationSystem.h"

namespace UE::Virtualization
{

bool PullPayloadsThreaded(TConstArrayView<FIoHash> PayloadIds, int32 BatchSize, const TCHAR* ProgressString, TUniqueFunction<void(const FPullRequest& Response)>&& Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PullPayloadsThreaded);
	
	IVirtualizationSystem& System = IVirtualizationSystem::Get();

	std::atomic<int32> NumCompletedPayloads = 0;

	const int32 NumPayloads = PayloadIds.Num();

	// This seems to be the sweet spot when it comes to our internal infrastructure, so use that as the default.
	const int32 MaxConcurrentTasks = 16;

	// We always want to leave at least one foreground worker free to avoid saturation. If we issue too many
	// concurrent task then we can potentially cause the DDC/Zen to be unable to run clean up tasks for long 
	// periods of times which can cause quite high memory spikes.
	const int32 ConcurrentTasks = FMath::Min(MaxConcurrentTasks, FTaskGraphInterface::Get().GetNumWorkerThreads() - 1);

	UE_LOGF(LogVirtualization, Display, "Will run up to %d pull batches concurrently", ConcurrentTasks);

	FWorkQueue WorkQueue(PayloadIds, BatchSize);

	UE::Tasks::FTaskEvent Event(UE_SOURCE_LOCATION);

	std::atomic<int32> NumTasks = 0;
	double LogTimer = FPlatformTime::Seconds();

	bool bHasErrors = false;

	while (NumTasks != 0 || !WorkQueue.IsEmpty())
	{
		int32 NumTaskAllowed = ConcurrentTasks - NumTasks;
		while (NumTaskAllowed > 0 && !WorkQueue.IsEmpty())
		{
			NumTasks++;

			UE::Tasks::Launch(UE_SOURCE_LOCATION,
				[Job = WorkQueue.GetJob(), &System, &NumCompletedPayloads, &NumTasks, &Event, &bHasErrors , &Callback]()
				{
					TArray<FPullRequest> Requests = ToRequestArray(Job);

					if (!System.PullData(Requests))
					{
						bHasErrors = true;
					}
					
					for (const FPullRequest& Request : Requests)
					{
						Callback(Request);
					}
					
					NumCompletedPayloads += Requests.Num();
					--NumTasks;

					Event.Trigger();
				});

			--NumTaskAllowed;
		}

		Event.Wait(FTimespan::FromSeconds(30.0));

		if (FPlatformTime::Seconds() - LogTimer >= 30.0)
		{
			const int32 CurrentCompletedPayloads = NumCompletedPayloads;
			const float Progress = ((float)CurrentCompletedPayloads / (float)NumPayloads) * 100.0f;

			UE_LOGF(LogVirtualization, Display, "%ls Cached %d/%d (%.1f%%)", 
				ProgressString != nullptr? ProgressString : TEXT("Processed"),
				CurrentCompletedPayloads, NumPayloads, Progress);

			LogTimer = FPlatformTime::Seconds();
		}
	}

	return !bHasErrors;
}

TArray<FString> FindPackages(EFindPackageFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindPackages);

	UE_LOGF(LogVirtualization, Display, "Searching for packages under the current project...");

	TArray<FString> PackagePaths;

	FAssetRegistryModule& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Do an async search even though we immediately block on it. This will result in the asset registry cache
	// being saved to disk on a background thread which is an operation we don't need to wait on. This can 
	// save a fair amount of time on larger projects.
	const bool bSynchronousSearch = false;
	AssetRegistry.Get().SearchAllAssets(bSynchronousSearch);
	AssetRegistry.Get().WaitForCompletion();

	const FString EnginePath = FPaths::EngineDir();

	const bool bFilterEngineContent = Flags & EFindPackageFlags::ExcludeEngineContent;

	AssetRegistry.Get().EnumerateAllPackages([&PackagePaths, EnginePath, bFilterEngineContent](FName PackageName, const FAssetPackageData& PackageData)
		{
			if (PackageData.CookedHash.IsValid())
			{
				return; // Skip cooked packages as they won't have a FPackageTrailer
			}
	
			FString RelFileName;
			if (PackageData.Extension != EPackageExtension::Unspecified && PackageData.Extension != EPackageExtension::Custom)
			{
				const FString Extension = LexToString(PackageData.Extension);
				if (FPackageName::TryConvertLongPackageNameToFilename(PackageName.ToString(), RelFileName, Extension))
				{
					FString StdFileName = FPaths::CreateStandardFilename(RelFileName);
				
					// Now we have the absolute file path we can filter out engine packages
					if (!bFilterEngineContent || !StdFileName.StartsWith(EnginePath))
					{
						PackagePaths.Emplace(MoveTemp(StdFileName));
					}
				}
			}
		});

	return PackagePaths;
}

TArray<FString> FindPackagesInDirectory(const FString& DirectoryToSearch)
{
	UE_LOGF(LogVirtualization, Display, "Searching for packages under '%ls'...", *DirectoryToSearch);

	TArray<FString> FilesInPackageFolder;
	FPackageName::FindPackagesInDirectory(FilesInPackageFolder, DirectoryToSearch);

	TArray<FString> PackageNames;
	PackageNames.Reserve(FilesInPackageFolder.Num());

	for (const FString& BasePath : FilesInPackageFolder)
	{
		PackageNames.Add(FPaths::CreateStandardFilename(BasePath));
	}

	return PackageNames;
}

TArray<FString> DiscoverPackages(const FString& CmdlineParams, EFindPackageFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DiscoverPackages);

	FString PackageDirectories;
	if (FParse::Value(*CmdlineParams, TEXT("PackageDir="), PackageDirectories) || FParse::Value(*CmdlineParams, TEXT("PackageFolder="), PackageDirectories))
	{
		TArray<FString> Packages;

		UE::String::ParseTokensMultiple(PackageDirectories, TConstArrayView<TCHAR>({ '+', ',' }),
			[&Packages](FStringView PackageDirectory)
			{
				TArray<FString> FoundPackages = FindPackagesInDirectory(FString(PackageDirectory));
				Packages.Append(FoundPackages);
			}, UE::String::EParseTokensOptions::Trim | UE::String::EParseTokensOptions::SkipEmpty);
		
		return Packages;
	}
	else
	{
		return FindPackages(Flags);
	}
}

TArray<FIoHash> FindVirtualizedPayloads(const TArray<FString>& PackagePaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindVirtualizedPayloads);

	// Each task will write out to its own TSet so we don't have to lock anything, we
	// will combine the sets at the end.
	TArray<TSet<FIoHash>> PayloadsPerTask;

	ParallelForWithTaskContext(PayloadsPerTask, PackagePaths.Num(),
		[&PackagePaths](TSet<FIoHash>& Context, int32 Index)
		{
			const FString& PackageName = PackagePaths[Index];

			UE::FPackageTrailer Trailer;
			if (UE::FPackageTrailer::TryLoadFromFile(PackageName, Trailer))
			{
				TArray<FIoHash> VirtualizedPayloads = Trailer.GetPayloads(UE::EPayloadStorageType::Virtualized);
				Context.Append(VirtualizedPayloads);
			}
		});

	// Combine the results into a final set
	TSet<FIoHash> AllPayloads;
	for (const TSet<FIoHash>& Payloads : PayloadsPerTask)
	{
		AllPayloads.Append(Payloads);
	}

	return AllPayloads.Array();
}

void FindVirtualizedPayloadsAndTrailers(const TArray<FString>& PackagePaths, TMap<FString, UE::FPackageTrailer>& OutPackages, TSet<FIoHash>& OutPayloads)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindVirtualizedPayloadsAndTrailers);

	struct FTaskContext
	{
		TMap<FString, UE::FPackageTrailer> Packages;
		TSet<FIoHash> Payloads;
	};

	TArray<FTaskContext> TaskContext;

	ParallelForWithTaskContext(TaskContext, PackagePaths.Num(),
		[&PackagePaths](FTaskContext& Context, int32 Index)
		{
			const FString& PackageName = PackagePaths[Index];

			UE::FPackageTrailer Trailer;
			if (UE::FPackageTrailer::TryLoadFromFile(PackageName, Trailer))
			{
				const TArray<FIoHash> VirtualizedPayloads = Trailer.GetPayloads(UE::EPayloadStorageType::Virtualized);
				if (!VirtualizedPayloads.IsEmpty())
				{
					Context.Packages.Emplace(PackageName, MoveTemp(Trailer));
					Context.Payloads.Append(VirtualizedPayloads);
				}
			}
		});

	for (const FTaskContext& Context : TaskContext)
	{
		OutPackages.Append(Context.Packages);
		OutPayloads.Append(Context.Payloads);
	}
}

TArray<FPullRequest> ToRequestArray(TConstArrayView<FIoHash> IdentifierArray)
{
	TArray<FPullRequest> Requests;
	Requests.Reserve(IdentifierArray.Num());

	for (const FIoHash& Id : IdentifierArray)
	{
		Requests.Emplace(FPullRequest(Id));
	}

	return Requests;
}

FWorkQueue::FWorkQueue(const TConstArrayView<FIoHash> InWork, int32 JobSize)
	: Work(InWork)
{
	CreateJobs(JobSize);
}

FWorkQueue::FWorkQueue(TArray<FIoHash>&& InWork, int32 JobSize)
	: Work(InWork)
{
	CreateJobs(JobSize);
}

void FWorkQueue::CreateJobs(int32 JobSize)
{
	const int32 NumJobs = FMath::DivideAndRoundUp<int32>(Work.Num(), JobSize);
	Jobs.Reserve(NumJobs);

	for (int32 JobIndex = 0; JobIndex < NumJobs; ++JobIndex)
	{
		const int32 JobStart = JobIndex * JobSize;
		const int32 JobEnd = FMath::Min((JobIndex + 1) * JobSize, Work.Num());

		FJob Job = MakeArrayView(&Work[JobStart], JobEnd - JobStart);
		Jobs.Add(Job);
	}
}

FWorkQueue::FJob FWorkQueue::GetJob()
{
	if (!Jobs.IsEmpty())
	{
		return Jobs.Pop(EAllowShrinking::No);
	}
	else
	{
		return FJob();
	}
}

bool FWorkQueue::IsEmpty() const
{
	return Jobs.IsEmpty();
}

} //namespace UE::Virtualization
