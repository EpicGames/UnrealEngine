// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"

namespace UE { class FPackageTrailer; }
namespace UE::Virtualization { struct FPullRequest; }
struct FIoHash;

namespace UE::Virtualization // Utility functions
{

/*
 * Utility to pull payloads in batches from many threads at once which can be much faster than pulling from a single thread
 * in a single batch. The provided callback will be invoked each time that a request is completed.
 * Every 30 seconds an update will be printed to the log showing how many payloads have been processed so that the user does
 * not think that the process has stalled.
 * 
 * @param PayloadIds		The payloads that should be pulled
 * @param BatchSize			The max number of payload ids that should be in each batch
 * @param ProgressString	An optional string to be printed as part of the status shown every 30 seconds, can be nullptr.
 * @param Callback			Called on every request once it has been processed.
 * 
 * @ return True if all requests succeeded, false if there were failures.
 */
bool PullPayloadsThreaded(TConstArrayView<FIoHash> PayloadIds, int32 BatchSize, const TCHAR* ProgressString, TUniqueFunction<void(const UE::Virtualization::FPullRequest& Response)>&& Callback);

/** Used to customize package discovery behavior */
enum EFindPackageFlags
{
	None = 0,
	/** Do not return packages mounted to the engine or engine plugins */
	ExcludeEngineContent = 1 << 0
};

/** 
 * Parse all of the active mount points for the current project and returns the
 * packages found.
 */
TArray<FString> FindPackages(EFindPackageFlags Flags);

/** Finds all of the packages under a given directory including its subdirectories */
TArray<FString> FindPackagesInDirectory(const FString& DirectoryToSearch);

/**
 * Finds all of the packages under a the directory given by the provided command line.
 * If no commandline switch can be found then the function will return all available 
 * packages.
 * Valid commandline switches:
 * '-PackageDir=...'
 * '-PackageFolder=...'
 * 
 * @param CmdlineParams A string containing the command line
 * @return An array with the file path for each package found
 */
TArray<FString> DiscoverPackages(const FString& CmdlineParams, EFindPackageFlags Flags);

/** Returns a combined list of unique virtualized payload ids from the given list of packages */
TArray<FIoHash> FindVirtualizedPayloads(const TArray<FString>& PackageNames);

/** 
 * Load and parse the package trailers for the given packages. We return a map of all of the packages that contain
 * virtualized payloads and a unique set of all the virtualized payloads referenced. Note that packages can reference
 * the same payload if they re-use assets.
 * 
 * @param PackagePaths	A list of packages to check
 * @param OutPackages	A map of package paths to package trailers
 * @param OutPayloads	A unique set of all the virtualized payloads referenced by the packages
 */
void FindVirtualizedPayloadsAndTrailers(const TArray<FString>& PackagePaths, TMap<FString, UE::FPackageTrailer>& OutPackages, TSet<FIoHash>& OutPayloads);

/** Utility to turn an array of FIoHash into an array of FPullRequest */
TArray<FPullRequest> ToRequestArray(TConstArrayView<FIoHash> IdentifierArray);

} //namespace UE::Virtualization


namespace UE::Virtualization // Utility classes
{

/** 
 * Takes a list of FIoHashs and breaks them into smaller jobs based on the JobSize.
 * These jobs (in the form of an TArrayView) can then be requested one at a time
 * until there are no more jobs left.
 * The intended use is to make it easier to divide up the FIoHash list into smaller
 * units to be pushed to the taskgraph system.
 */
class FWorkQueue
{
public:
	using FJob = TArrayView<const FIoHash>;

	FWorkQueue(TConstArrayView<FIoHash> InWork, int32 JobSize);
	FWorkQueue(TArray<FIoHash>&& InWork, int32 JobSize);
	~FWorkQueue() = default;

	/** 
	 * Returns a FJob containing FIoHashes to be processed. 
	 * If there is no more work to be done then empty jobs will be returned 
	 * 
	 * NOTE: That FJob is only valid for the lifespan of the FWorkQueue
	 */
	FJob GetJob();

	/** Returns true if there is no more work to done */
	bool IsEmpty() const;

private:

	void CreateJobs(int32 JobSize);

	TArray<FIoHash> Work;
	TArray<FJob> Jobs;
};

} //namespace UE::Virtualization
