// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleCommandUtils.h"

#include "HAL/FileManager.h"
#include "Misc/PackagePath.h"
#include "Serialization/Archive.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PackageResourceManager.h"

namespace UE::Virtualization
{

void LogPackageTrailerLoadingError(FArchive* Ar, const TCHAR* DebugName)
{
	if (Ar == nullptr)
	{
		UE_LOGF(LogVirtualization, Error, "Could not find the package file: '%ls'", DebugName);
		return;
	}

	FPackageFileSummary Summary;
	*Ar << Summary;

	if (Ar->IsError())
	{
		UE_LOGF(LogVirtualization, Error, "Could not find load the package summary from disk: '%ls'", DebugName);
		return;
	}

	if (Summary.Tag != PACKAGE_FILE_TAG)
	{
		UE_LOGF(LogVirtualization, Error, "Package summary seems to be corrupted: '%ls'", DebugName);
		return;
	}

	int32 PackageVersion = Summary.GetFileVersionUE().ToValue();
	if (PackageVersion >= (int32)EUnrealEngineObjectUE5Version::PAYLOAD_TOC)
	{
		UE_LOGF(LogVirtualization, Error, "Package trailer is missing from the package file: '%ls'", DebugName);
		return;
	}

	UE_LOGF(LogVirtualization, Error, "Package is tool old (version %d) to have a package trailer (version %d): '%ls'", PackageVersion, int(EUnrealEngineObjectUE5Version::PAYLOAD_TOC), DebugName);
}

void LogPackageTrailerLoadingError(const FPackagePath& Path)
{
	const FString DebugName = Path.GetPackageName();
	TUniquePtr<FArchive> PackageAr = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, DebugName);

	LogPackageTrailerLoadingError(PackageAr.Get(), *DebugName);
}

void LogPackageTrailerLoadingError(const FString& Path)
{
	TUniquePtr<FArchive> PackageAr(IFileManager::Get().CreateFileReader(*Path));

	LogPackageTrailerLoadingError(PackageAr.Get(), *Path);
}

TArray<TPair<FString, UE::FPackageTrailer>> LoadPackageTrailerFromArgs(const TArray<FString>& Args)
{
	TArray<TPair<FString, UE::FPackageTrailer>> Packages;
	Packages.Reserve(Args.Num());

	for (const FString& Arg : Args)
	{
		FString PathString;

		if (FPackageName::ParseExportTextPath(Arg, nullptr /*OutClassName*/, &PathString))
		{
			PathString = FPackageName::ObjectPathToPackageName(PathString);
		}
		else
		{
			PathString = Arg;
		}

		FPackageTrailer Trailer;

		FPackagePath Path;
		if (FPackagePath::TryFromMountedName(PathString, Path))
		{
			if (!FPackageTrailer::TryLoadFromPackage(Path, Trailer))
			{
				LogPackageTrailerLoadingError(Path);
				continue;
			}
		}
		else if (IFileManager::Get().FileExists(*PathString))
		{
			// IF we couldn't turn it into a FPackagePath it could be a path to a package not under any current mount point.
			// So for a final attempt we will see if we can find the file on disk and load the package trailer that way.

			if (!FPackageTrailer::TryLoadFromFile(PathString, Trailer))
			{
				LogPackageTrailerLoadingError(PathString);
				continue;
			}
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "Arg '%ls' could not be converted to a valid package path", *Arg);
			continue;
		}

		Packages.Add({ MoveTemp(PathString) , MoveTemp(Trailer) });	
	}

	return Packages;
}

} // namespace UE::Virtualization
