// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/PackageTrailer.h"

class FPackagePath;

namespace UE::Virtualization
{

TArray<TPair<FString, UE::FPackageTrailer>> LoadPackageTrailerFromArgs(const TArray<FString>& Args);

void LogPackageTrailerLoadingError(const FPackagePath& Path);
void LogPackageTrailerLoadingError(const FString& Path);

} //namespace UE::Virtualization
