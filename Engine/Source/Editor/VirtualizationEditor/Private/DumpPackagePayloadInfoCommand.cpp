// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleCommandUtils.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PackageTrailer.h"
#include "VirtualizationExperimentalUtilities.h"

#define LOCTEXT_NAMESPACE "Virtualization"

namespace UE::Virtualization
{

#if WITH_EDITORONLY_DATA

FString BytesToString(int64 SizeInBytes)
{
	if (SizeInBytes < (8 *1024))
	{
		return FString::Printf(TEXT("%4" INT64_FMT " bytes"), SizeInBytes);
	}
	else if (SizeInBytes < (1024 * 1024))
	{
		double SizeInKb = static_cast<double>(SizeInBytes) / (1024.0);
		return FString::Printf(TEXT("%.2f KB"), SizeInKb);
	}
	else
	{
		double SizeInMB = static_cast<double>(SizeInBytes) / (1024.0 * 1024.0);
		return FString::Printf(TEXT("%.2f MB"), SizeInMB);
	}
}

/**
 * This function is used to write information about package's payloads to the log file. This has no 
 * practical development use and should only be used for debugging purposes. 
 * 
 * @param Args	The function expects each arg to be a valid package path. Failure to provide a valid
 *				package path will result in errors being written to the log.
 */
void DumpPackagePayloadInfo(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		UE_LOGF(LogVirtualization, Error, "Command 'DumpPackagePayloadInfo' called without any arguments");
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(DumpPackagePayloadInfo);
	FScopedSlowTask Progress(0.0f, LOCTEXT("VAFindPayloadInfo", "Finding payload info..."));

	for (const FString& Arg : Args)
	{
		FString PathString;

		if (FPackageName::ParseExportTextPath(Arg, nullptr /*OutClassName*/, &PathString))
		{
			PathString = FPackageName::ObjectPathToPackageName(PathString);
		}
		else
		{
			PathString = Arg;
		}

		FPackageTrailer Trailer;

		FPackagePath Path;
		if (FPackagePath::TryFromMountedName(PathString, Path))
		{	
			if (!FPackageTrailer::TryLoadFromPackage(Path, Trailer))
			{
				LogPackageTrailerLoadingError(Path);
				continue;
			}
		}
		else if (IFileManager::Get().FileExists(*PathString))
		{
			// IF we couldn't turn it into a FPackagePath it could be a path to a package not under any current mount point.
			// So for a final attempt we will see if we can find the file on disk and load the package trailer that way.
			
			if (!FPackageTrailer::TryLoadFromFile(PathString, Trailer))
			{
				LogPackageTrailerLoadingError(PathString);
				continue;
			}	
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "Arg '%ls' could not be converted to a valid package path", *Arg);
			continue;
		}

		TArray<FIoHash> LocalPayloadIds = Trailer.GetPayloads(UE::EPayloadStorageType::Local);
		TArray<FIoHash> VirtualizedPayloadIds = Trailer.GetPayloads(UE::EPayloadStorageType::Virtualized);

		UE_LOGF(LogVirtualization, Display, ""); // Blank line to make the output easier to read
		UE_LOGF(LogVirtualization, Display, "Package: '%ls' has %d local and %d virtualized payloads", *Path.GetDebugName(), LocalPayloadIds.Num(), VirtualizedPayloadIds.Num());

		if (LocalPayloadIds.Num() > 0)
		{
			UE_LOGF(LogVirtualization, Display, "LocalPayloads:");
			UE_LOGF(LogVirtualization, Display, "Index | %-40ls | SizeOnDisk | FilterReason", TEXT("PayloadIdentifier"));
			for (int32 Index = 0; Index < LocalPayloadIds.Num(); ++Index)
			{
				FPayloadInfo Info = Trailer.GetPayloadInfo(LocalPayloadIds[Index]);

				Info.FilterFlags = UE::Virtualization::Utils::FixFilterFlags(PathString, Info.CompressedSize, Info.FilterFlags);

				UE_LOGF(LogVirtualization, Display, "%02d    | %ls | %-10ls | %ls",
					Index,
					*LexToString(LocalPayloadIds[Index]),
					*BytesToString(Info.CompressedSize),
					*LexToString(Info.FilterFlags));
			}
		}

		if (VirtualizedPayloadIds.Num() > 0)
		{
			UE_LOGF(LogVirtualization, Display, "VirtualizedPayloads:");
			UE_LOGF(LogVirtualization, Display, "Index|\t%-40ls|\tFilterReason", TEXT("PayloadIdentifier"));
			for (int32 Index = 0; Index < VirtualizedPayloadIds.Num(); ++Index)
			{
				FPayloadInfo Info = Trailer.GetPayloadInfo(VirtualizedPayloadIds[Index]);
				UE_LOGF(LogVirtualization, Display, "%02d:  |\t%ls|\t%ls", Index, *LexToString(VirtualizedPayloadIds[Index]), *LexToString(Info.FilterFlags));
			}
		}
	}
}

/** 
 * Note that this command is only valid when 'WITH_EDITORONLY_DATA 1' as virtualized payloads are not
 * expected to exist at runtime. 
 */
static FAutoConsoleCommand CCmdDumpPayloadTrailer = FAutoConsoleCommand(
	TEXT("DumpPackagePayloadInfo"),
	TEXT("Writes out information about a package's payloads to the log."),
	FConsoleCommandWithArgsDelegate::CreateStatic(DumpPackagePayloadInfo));

#endif //WITH_EDITORONLY_DATA

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE
