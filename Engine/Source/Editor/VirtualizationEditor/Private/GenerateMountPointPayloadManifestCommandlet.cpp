// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateMountPointPayloadManifestCommandlet.h"

#include "Async/ParallelFor.h"
#include "CommandletUtils.h"
#include "HAL/FileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/PathViews.h"
#include "UObject/PackageTrailer.h"
#include "VirtualizationExperimentalUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateMountPointPayloadManifestCommandlet)

#if 1
	#define UE_OUTPUTBYTES(x) ((double)x / (1024.0 * 1024.0 * 1024.0))
#else
	#define UE_OUTPUTBYTES(x)
#endif

enum class EOutputFlags : uint32
{
	None					= 0,
	DetailedFilterReasons	= 1 << 0,
	MountPointPaths			= 1 << 1
};

ENUM_CLASS_FLAGS(EOutputFlags);

struct FMountPointStatistics
{
	FString RootPath;

	uint64 TotalFileSize = 0;
	uint64 NumFiles = 0;
	uint64 NumFilesWithPayloads = 0;
	uint64 NumFilesWithPendingPayloads = 0;

	uint64 PendingPayloadCount = 0;
	uint64 FilteredPayloadCount = 0;

	uint64 LocalPendingSize = 0;
	uint64 LocalFilteredSize = 0;

	/** Used when 'DetailedFilterReasons' cmdline switch is used */
	struct FilteredDetails
	{
	public:
		void AddFile(UE::Virtualization::EPayloadFilterReason Reason, uint64 FileSize)
		{
			if (Reason == UE::Virtualization::EPayloadFilterReason::None)
			{
				NumFiles[0]++;
				TotalFileSize[0] += FileSize;
			}
			else
			{
				for (uint16 Index = 1; Index < UE::Virtualization::NumPayloadFilterReasons; ++Index)
				{
					if (((uint16)Reason & (1 << (Index-1))) != 0)
					{
						NumFiles[Index]++;
						TotalFileSize[Index] += FileSize;
					}
				}
			}
		}

		uint64 GetCount(uint16 FilterReasonIndex) const
		{
			check(FilterReasonIndex < UE::Virtualization::NumPayloadFilterReasons);
			return NumFiles[FilterReasonIndex];
		}

		uint64 GetTotalSize(uint16 FilterReasonIndex) const
		{
			check(FilterReasonIndex < UE::Virtualization::NumPayloadFilterReasons);
			return TotalFileSize[FilterReasonIndex];
		}

		FilteredDetails& operator+=(const FilteredDetails& Other)
		{
			for (uint16 Index = 0; Index < UE::Virtualization::NumPayloadFilterReasons; ++Index)
			{
				NumFiles[Index] += Other.NumFiles[Index];
				TotalFileSize[Index] += Other.TotalFileSize[Index];
			}
			return *this;
		}

	private:
		uint64 NumFiles[UE::Virtualization::NumPayloadFilterReasons] = { 0 };
		uint64 TotalFileSize[UE::Virtualization::NumPayloadFilterReasons] = { 0 };
	} FilteredDetails;

	FMountPointStatistics& operator+=(const FMountPointStatistics& Other)
	{
		TotalFileSize += Other.TotalFileSize;
		NumFiles += Other.NumFiles;
		NumFilesWithPayloads += Other.NumFilesWithPayloads;
		NumFilesWithPendingPayloads += Other.NumFilesWithPendingPayloads;

		PendingPayloadCount += Other.PendingPayloadCount;
		FilteredPayloadCount += Other.FilteredPayloadCount;

		LocalPendingSize += Other.LocalPendingSize;
		LocalFilteredSize += Other.LocalFilteredSize;

		FilteredDetails += Other.FilteredDetails;

		return *this;
	}
};

FMountPointStatistics ProcessMountPoint(FStringView RootPath, const TArray<FString>& FilePaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessMountPoint);

	FMountPointStatistics Stats;
	Stats.RootPath = RootPath;

	TArray<FMountPointStatistics> ContextStats;

	ParallelForWithTaskContext(ContextStats, FilePaths.Num(),
		[&FilePaths](FMountPointStatistics& Stats, int32 Index)
		{
			const FString& FilePath = FilePaths[Index];
			int64 FileSize = IFileManager::Get().FileSize(*FilePath);
			if (FileSize == INDEX_NONE)
			{
				UE_LOGF(LogVirtualization, Error, "Unable to find file '%ls'", *FilePath);
				return;
			}

			Stats.NumFiles++;
			Stats.TotalFileSize += FileSize;

			UE::FPackageTrailer Trailer;
			if (UE::FPackageTrailer::TryLoadFromFile(FilePath, Trailer))
			{
				if (Trailer.GetNumPayloads(UE::EPayloadStorageType::Any) > 0)
				{
					Stats.NumFilesWithPayloads++;
				}

				bool bHasPendingPayloads = false;

				Trailer.ForEachPayload([&FilePath, &Stats, &bHasPendingPayloads](const FIoHash& Id, uint64 SizeOnDisk, uint64 RawSize, UE::EPayloadAccessMode Mode, UE::Virtualization::EPayloadFilterReason Filter)->void
					{
						if (Mode == UE::EPayloadAccessMode::Local)
						{
							Filter = UE::Virtualization::Utils::FixFilterFlags(FilePath, SizeOnDisk, Filter);

							Stats.FilteredDetails.AddFile(Filter, SizeOnDisk);

							if (Filter == UE::Virtualization::EPayloadFilterReason::None)
							{
								Stats.PendingPayloadCount++;
								Stats.LocalPendingSize += SizeOnDisk;

								bHasPendingPayloads = true;
							}
							else
							{
								Stats.FilteredPayloadCount++;
								Stats.LocalFilteredSize += SizeOnDisk;
							}
						}
					});

				if (bHasPendingPayloads)
				{
					Stats.NumFilesWithPendingPayloads++;
				}
			}
		});

	for (const FMountPointStatistics& Context : ContextStats)
	{
		Stats += Context;
	}

	return Stats;
}

FAnsiStringBuilderBase& FormatByteOutput(FAnsiStringBuilderBase& Builder, uint64 Bytes, EByteFormatting ByteFormat)
{
	switch (ByteFormat)
	{
		case EByteFormatting::Bytes:
			Builder << Bytes;
			break;
		case EByteFormatting::MiB:
			Builder << (double)Bytes / (1024.0 * 1024.0);
			break;
		case EByteFormatting::GiB:
			Builder << (double)Bytes / (1024.0 * 1024.0 * 1024.0);
			break;
		default:
			checkNoEntry();

	}

	return Builder;
}

bool TryLexFromString(FStringView String, EByteFormatting& OutFormat)
{
	if (String == TEXT("Bytes"))
	{
		OutFormat = EByteFormatting::Bytes;
		return true;
	}
	else if (String == TEXT("MiB"))
	{
		OutFormat = EByteFormatting::MiB;
		return true;
	}
	else if (String == TEXT("GiB"))
	{
		OutFormat = EByteFormatting::GiB;
		return true;
	}
	else
	{
		return false;
	}
}

bool OutputMountPointStatistics(const FString& OutputFilePath, const TArray<FMountPointStatistics>& Statistics, EOutputFlags Flags, EByteFormatting ByteFormat)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OutputMountPointStatistics);

	TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*OutputFilePath));
	if (!Ar.IsValid())
	{
		UE_LOGFMT(LogVirtualization, Error, "Failed to open '{OutputFilePath}' for writing", OutputFilePath);
		return false;
	}

	{
		TAnsiStringBuilder<512> Heading;
		Heading = "Name,NumFiles,NumFilesWithPayloads,PendingPayloadCount,FilteredPayloadCount,VirtualizedPercent,TotalFileSize,StructuredDataSize,PendingPayloadSize,FilteredPayloadSize";

		if (EnumHasAllFlags(Flags, EOutputFlags::DetailedFilterReasons))
		{
			for (uint16 FilterIdx = 1; FilterIdx < UE::Virtualization::NumPayloadFilterReasons; ++FilterIdx)
			{
				Heading << ",Filter (" << LexToString((UE::Virtualization::EPayloadFilterReason)(1 << (FilterIdx - 1))) << ") Size";
			}
		}

		if (EnumHasAllFlags(Flags, EOutputFlags::MountPointPaths))
		{
			Heading << ",Path";
		}

		Heading << "\n";

		Ar->Serialize((void*)Heading.GetData(), Heading.Len() * sizeof(FAnsiStringView::ElementType));

		if (Ar->IsError())
		{
			UE_LOGFMT(LogVirtualization, Error, "Failed to write out csv headers to '{OutputFilePath}'", OutputFilePath);
			return false;
		}
	}

	for (const FMountPointStatistics& Stats : Statistics)
	{
		const double VirtualizedPercent = Stats.NumFiles > 0 ? ((double)(Stats.NumFiles - Stats.NumFilesWithPendingPayloads) / (double)Stats.NumFiles) * 100.0 : 0.0;
		const uint64 StructuredDataSize = Stats.TotalFileSize - (Stats.LocalPendingSize + Stats.LocalFilteredSize);

		TAnsiStringBuilder<256> Line;
		Line << "\"" << FPathViews::GetPathLeaf(Stats.RootPath) << "\",";

		Line << Stats.NumFiles << ",";
		Line << Stats.NumFilesWithPayloads << ",";

		Line << Stats.PendingPayloadCount << ",";
		Line << Stats.FilteredPayloadCount << ",";

		Line.Appendf("%.1f,", VirtualizedPercent);

		FormatByteOutput(Line, Stats.TotalFileSize, ByteFormat) << ",";
		FormatByteOutput(Line, StructuredDataSize, ByteFormat) << ",";
		FormatByteOutput(Line, Stats.LocalPendingSize, ByteFormat) << ",";
		FormatByteOutput(Line, Stats.LocalFilteredSize, ByteFormat);

		if (EnumHasAllFlags(Flags, EOutputFlags::DetailedFilterReasons))
		{
			for (uint16 FilterIdx = 1; FilterIdx < UE::Virtualization::NumPayloadFilterReasons; ++FilterIdx)
			{
				Line << ",";
				FormatByteOutput(Line, Stats.FilteredDetails.GetTotalSize(FilterIdx), ByteFormat);
			}
		}

		if (EnumHasAllFlags(Flags, EOutputFlags::MountPointPaths))
		{
			Line << ",\"" << Stats.RootPath << "\"";
		}

		Line << "\n";

		Ar->Serialize((void*)Line.GetData(), Line.Len() * sizeof(FAnsiStringView::ElementType));

		if (Ar->IsError())
		{
			UE_LOGFMT(LogVirtualization, Error, "Failed to write csv data to '{OutputFilePath}'", OutputFilePath);
			return false;
		}
	}

	Ar.Reset();

	UE_LOGFMT(LogVirtualization, Display, "Wrote output to : '{OutputFilePath}'", OutputFilePath);
	return true;
}

UGenerateMountPointPayloadManifestCommandlet::UGenerateMountPointPayloadManifestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UGenerateMountPointPayloadManifestCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateMountPointPayloadManifestCommandlet);

	if (!TryParseCmdline(Params))
	{
		UE_LOGF(LogVirtualization, Error, "Failed to parse the command line correctly");
		return -1;
	}

	UE_LOGF(LogVirtualization, Display, "Generating mount point summary for all files...");

	TArray<FString> PackageNames = UE::Virtualization::DiscoverPackages(Params, UE::Virtualization::EFindPackageFlags::ExcludeEngineContent);

	UE_LOGF(LogVirtualization, Display, "Found %d files to look in", PackageNames.Num());

	// RootPath of the mount point, array of file paths under that mount point
	TMap<FString, TArray<FString>> MountPointMap;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SortingMountPoints);
		UE_LOGF(LogVirtualization, Display, "Sorting by mount point...");

		for (FString& Path : PackageNames)
		{
			int32 ContentIndex = Path.Find(TEXT("/Content/"));
			if (ContentIndex != INDEX_NONE)
			{
				int32 MountPointIndex = Path.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd, ContentIndex);
				if (MountPointIndex != INDEX_NONE)
				{
					MountPointIndex++;

					FStringView MountPoint = FStringView(Path).SubStr(0, ContentIndex);

					if (TArray<FString>* ExistingPaths = MountPointMap.FindByHash(GetTypeHash(MountPoint), MountPoint))
					{
						ExistingPaths->Add(MoveTemp(Path));
					}
					else
					{
						TArray<FString> NewPaths;
						NewPaths.Add(MoveTemp(Path));

						MountPointMap.Add(FString(MountPoint), MoveTemp(NewPaths));
					}
				}
			}
			else
			{
				UE_LOGF(LogVirtualization, Warning, "Package '%ls' not under a valid content directory, skipping!", *Path);
			}
		}
	}

	UE_LOGF(LogVirtualization, Display, "Found %d mountpoints", MountPointMap.Num());

	TArray<FMountPointStatistics> Stats;
	Stats.Reserve(MountPointMap.Num());

	UE_LOGF(LogVirtualization, Display, "Processing mountpoints...");
	for (const TPair<FString, TArray<FString>>& It : MountPointMap)
	{
		FMountPointStatistics MountPointStats = ProcessMountPoint(It.Key, It.Value);
		Stats.Add(MountPointStats);
	}

	UE_LOGF(LogVirtualization, Display, "Processing mountpoints completed");

	EOutputFlags Flags = EOutputFlags::None;
	if (bDetailedFilterReasons)
	{
		Flags |= EOutputFlags::DetailedFilterReasons;
	}

	if (bOutputPaths)
	{
		Flags |= EOutputFlags::MountPointPaths;
	}

	if (!OutputMountPointStatistics(OutputFilePath, Stats, Flags, ByteFormat))
	{
		return -1;
	}

	return 0;
}

bool UGenerateMountPointPayloadManifestCommandlet::TryParseCmdline(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	if (Switches.Contains(TEXT("DetailedFilterReasons")))
	{
		bDetailedFilterReasons = true;
	}

	if (Switches.Contains(TEXT("NoPaths")))
	{
		bOutputPaths = false;
	}

	FString ByteFormatString;
	if (FParse::Value(*Params, TEXT("ByteFormat="), ByteFormatString))
	{
		if (!TryLexFromString(ByteFormatString, ByteFormat))
		{
			UE_LOGFMT(LogVirtualization, Error, "'{Arg}' is not a valid value for  -ByteFormat=", ByteFormatString);
			return false;
		}
	}

	if (!TryParseOutputPathFromCmdline(Params))
	{
		return false;
	}

	return true;
}

bool UGenerateMountPointPayloadManifestCommandlet::TryParseOutputPathFromCmdline(const FString& Params)
{
	// First see the entire file path has been provided on the commandline
	FParse::Value(*Params, TEXT("OutputPath="), OutputFilePath);

	// Now check to see if just the filename has been provided on the commandline
	FString CustomOutputName;
	if (FParse::Value(*Params, TEXT("OutputName="), CustomOutputName))
	{
		if (!OutputFilePath.IsEmpty() && !CustomOutputName.IsEmpty())
		{
			UE_LOGFMT(LogVirtualization, Error, "It is invalid to set both '-OutputPath=' and '-OutputName=' for the same command");
			return false;
		}

		TStringBuilder<512> Builder;
		Builder << FPaths::ProjectSavedDir() << TEXT("PayloadManifest/") << CustomOutputName;

		OutputFilePath = Builder.ToView();
	}

	if (OutputFilePath.IsEmpty())
	{
		TStringBuilder<512> Builder;
		Builder << FPaths::ProjectSavedDir() << TEXT("PayloadManifest/mountpoints.csv");

		OutputFilePath = Builder.ToView();
	}

	FPaths::NormalizeFilename(OutputFilePath);

	UE_LOGFMT(LogVirtualization, Display, "Final output will be written to {OutputFilePath}", OutputFilePath);

	return true;
}
