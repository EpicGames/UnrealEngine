// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "UObject/ObjectMacros.h"

#include "GenerateMountPointPayloadManifestCommandlet.generated.h"

enum class EByteFormatting
{
	Bytes = 0,
	MiB,
	GiB
};

/**
 * Because the commandlet is the VirtualizationEditor module it needs to be invoked 
 * with the command line:
 * -run="VirtualizationEditor.GenerateMountPointPayloadManifestCommandlet"
 * 
 * By default the final output will be written to:
 * <project root>/saved/PayloadManifest/mountpoints.csv
 * 
 * Additional args:
 * -DetailedFilterReasons
 *     This switch will provide a breakdown of how much content per mount point
 *     is prevented from being virtualized by a specific filter.
 * -NoPaths
 *     This switch will prevent the relative path of the mount point being written
 *     to the output.
 * -ByteFormat=Bytes/MiB/GiB
 *     This value allows the caller to set how bytes should be formatted to the
 *     csv file. The default is to output raw bytes.
 * -OutputPath=""
 *     The full path (including filename and extension) of where to write the
 *     final output.
 * -OutputName=""
 *     The file name (including extension) of where to write the final output.
 *     Note that the file will be written to the default output directory.
 */
UCLASS()
class UGenerateMountPointPayloadManifestCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);

private:

	bool TryParseCmdline(const FString& Params);
	bool TryParseOutputPathFromCmdline(const FString& Params);

	FString OutputFilePath;

	bool bDetailedFilterReasons = false;
	bool bOutputPaths = true;

	EByteFormatting ByteFormat = EByteFormatting::Bytes;
};
