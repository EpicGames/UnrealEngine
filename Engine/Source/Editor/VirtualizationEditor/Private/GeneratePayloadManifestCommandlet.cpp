// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneratePayloadManifestCommandlet.h"

#include "CommandletUtils.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/Paths.h"
#include "UObject/PackageTrailer.h"
#include "VirtualizationExperimentalUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeneratePayloadManifestCommandlet)

ENUM_CLASS_FLAGS(UGeneratePayloadManifestCommandlet::EPayloadFilter);

class FSpreadSheet
{
public:

	bool OpenNewSheet()
	{
		if (NumSheets == 0)
		{
			if (!CleanupExistingFiles())
			{
				return false;
			}
		}

		Ar.Reset();

		TStringBuilder<512> CSVPath;
		CreateFilePath(++NumSheets, CSVPath);

		Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(CSVPath.ToString()));
		if (Ar.IsValid())
		{
			const ANSICHAR* Headings = "Path,PayloadId,SizeOnDisk,UncompressedSize,StorageType, FilterReason\n";
			Ar->Serialize((void*)Headings, FPlatformString::Strlen(Headings));

			return true;
		}
		else
		{
			UE_LOGF(LogVirtualization, Error, "Failed to open '%ls' for write", CSVPath.ToString());
			return false;
		}
	}

	bool PrintRow(FAnsiStringView Row)
	{
		if (++NumRowsInSheet >= MaxNumRowsPerSheet || (Ar->Tell() + Row.Len()) >= MaxFileSize)
		{
			if (!OpenNewSheet())
			{
				return false;
			}

			NumRowsInSheet = 0;
		}

		Ar->Serialize((void*)Row.GetData(), Row.Len() * sizeof(FAnsiStringView::ElementType));

		return true;
	}

	void Flush()
	{
		Ar.Reset();
	}

	FString GetDebugInfo() const
	{
		TStringBuilder<1024> Output;

		Output << TEXT("Manifest is comprised of ") << NumSheets << TEXT(" sheet(s) written to:");

		for (int32 Index = 0; Index < NumSheets; ++Index)
		{
			TStringBuilder<512> Path;
			CreateFilePath(Index, Path);

			Output << LINE_TERMINATOR << Path;
		}

		return Output.ToString();
	}

private:

	bool CleanupExistingFiles()
	{
		int32 SheetToDelete = 1;
		TStringBuilder<512> Path;
		CreateFilePath(SheetToDelete, Path);

		while (IFileManager::Get().FileExists(Path.ToString()))
		{
			if (!IFileManager::Get().Delete(Path.ToString()))
			{
				return false;
			}

			CreateFilePath(++SheetToDelete, Path);
		}

		return true;
	}

	static void CreateFilePath(int32 Index, FStringBuilderBase& Path)
	{
		Path.Reset();
		Path << FPaths::ProjectSavedDir() << TEXT("PayloadManifest/sheet") << Index << TEXT(".csv");
	}

	// With any values above these we had trouble importing the finished cvs into various spread sheet programs
	const int64 MaxNumRowsPerSheet = 250000;
	const int64 MaxFileSize = 45 * 1024 * 1024;

	int32 NumSheets = 0;
	int32 NumRowsInSheet = 0;

	TUniquePtr<FArchive> Ar;
};

UGeneratePayloadManifestCommandlet::UGeneratePayloadManifestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UGeneratePayloadManifestCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGeneratePayloadManifestCommandlet);

	if (!ParseCmdline(Params))
	{
		UE_LOGF(LogVirtualization, Error, "Failed to parse the command line correctly");
		return -1;
	}

	TArray<FString> PackageNames = UE::Virtualization::DiscoverPackages(Params, UE::Virtualization::EFindPackageFlags::ExcludeEngineContent);

	UE_LOGF(LogVirtualization, Display, "Found %d files to look in", PackageNames.Num());

	int32 PackageTrailerCount = 0;
	int32 PayloadCount = 0;

	FSpreadSheet Sheet;
	if (!Sheet.OpenNewSheet())
	{
		return 1;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ParsePackageTrailers);

		UE_LOGF(LogVirtualization, Display, "Parsing files...");

		uint32 FilesParsed = 0;
		for (const FString& PackagePath : PackageNames)
		{
			if (FPackageName::IsPackageFilename(PackagePath))
			{
				UE::FPackageTrailer Trailer;
				if (UE::FPackageTrailer::TryLoadFromFile(PackagePath, Trailer))
				{
					PackageTrailerCount++;

					Trailer.ForEachPayload([&Sheet, &PackagePath, &PayloadCount, OutputFilter = Filter](const FIoHash& Id, uint64 SizeOnDisk, uint64 RawSize, UE::EPayloadAccessMode Mode, UE::Virtualization::EPayloadFilterReason Reason)->void
						{
							if (Mode != UE::EPayloadAccessMode::Virtualized)
							{
								if (EnumHasAllFlags(OutputFilter, EPayloadFilter::VirtualizedOnly))
								{
									return;
								}

								Reason = UE::Virtualization::Utils::FixFilterFlags(PackagePath, SizeOnDisk, Reason);

								if (EnumHasAllFlags(OutputFilter, EPayloadFilter::PendingOnly) && Reason != UE::Virtualization::EPayloadFilterReason::None)
								{
									return;
								}

								if (EnumHasAllFlags(OutputFilter, EPayloadFilter::FilteredOnly) && Reason == UE::Virtualization::EPayloadFilterReason::None)
								{
									return;
								}
							}
							else
							{
								if (EnumHasAnyFlags(OutputFilter, EPayloadFilter::LocalOnly))
								{
									return;
								}
							}

							PayloadCount++;

							TAnsiStringBuilder<256> LineBuilder;
							LineBuilder << PackagePath << "," << Id << "," << SizeOnDisk << "," << RawSize << "," << Mode << "," << *LexToString(Reason) <<"\n";
							
							Sheet.PrintRow(LineBuilder);
						});

				}
			}

			if (++FilesParsed % 10000 == 0)
			{
				float PercentCompleted = ((float)FilesParsed / (float)PackageNames.Num()) * 100.0f;
				UE_LOGF(LogVirtualization, Display, "Parsed %d/%d (%.0f%%)", FilesParsed, PackageNames.Num(), PercentCompleted);
			}
			
		}
	}

	Sheet.Flush();

	UE_LOGF(LogVirtualization, Display, "Found %d package trailers with %d payloads", PackageTrailerCount, PayloadCount);
	UE_LOGF(LogVirtualization, Display, "%ls", *Sheet.GetDebugInfo());

	return  0;
}

bool UGeneratePayloadManifestCommandlet::ParseCmdline(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	auto ApplyFilter = [this, &Switches](const TCHAR* FilterName, EPayloadFilter FilterType)->bool
		{
			if (Switches.Contains(FilterName))
			{
				if (Filter != EPayloadFilter::None)
				{
					UE_LOGF(LogVirtualization, Error, "Comandlet cannot have multiple filters active at the same time");
					return false;
				}

				Filter = FilterType;
			}

			return true;
		};

	TPair<const TCHAR*, EPayloadFilter> Filters[] =
	{ 
		{TEXT("LocalOnly"), EPayloadFilter::LocalOnly},
		{TEXT("PendingOnly"), EPayloadFilter::PendingOnly},
		{TEXT("FilteredOnly"), EPayloadFilter::FilteredOnly},
		{TEXT("VirtualizedOnly"), EPayloadFilter::VirtualizedOnly}
	};

	for (const TPair<const TCHAR*, EPayloadFilter>& It : Filters)
	{
		if (!ApplyFilter(It.Key, It.Value))
		{
			return false;
		}
	}

	return true;
}
