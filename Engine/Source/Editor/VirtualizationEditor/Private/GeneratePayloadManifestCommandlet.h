// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "UObject/ObjectMacros.h"

#include "GeneratePayloadManifestCommandlet.generated.h"

/**
 * Creates a csv file containing info about all of the payloads in a set of packages.
 * By default the commandlet will parse the payloads of all packages in the current
 * project but this can be overridden with the cmdline switch -PackageDir=XYZ which
 * will allow the commandlet to parse the payloads of the packages in a given directory.
 * 
 * Because the commandlet is the VirtualizationEditor module it needs to be invoked 
 * with the command line:
 * -run="VirtualizationEditor.GeneratePayloadManifest"
 * 
 * By default the final output will be written to:
 * <project root>/saved/PayloadManifest/sheet1.csv
 * note that if the csv file is large enough it will be split over several files.
 * 
 * Additional args:
 * -LocalOnly
 *     Only output payloads that are stored locally on disk.
 * -PendingOnly
 *     Only output payloads that are stored locally on disk but could be virtualized.
 * -FilteredOnly
 *     Only output payloads that are stored locally on disk but would never be virtualized
 *     due to one or more filters.
 * -VirtualizedOnly
 *     Only output payloads that have been virtualized.
 */
UCLASS()
class UGeneratePayloadManifestCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);

private:

	bool ParseCmdline(const FString& Params);

	enum class EPayloadFilter
	{
		None = 0,
		/* All locally stored payloads */
		LocalOnly		=  1 << 0,
		/* Locally stored payloads that could be virtualized */
		PendingOnly		= (1 << 1) | LocalOnly,
		/* Local payloads that are filtered from the virtualization process. */
		FilteredOnly	= (1 << 2) | LocalOnly,
		/* Virtualized payloads */
		VirtualizedOnly	= (1 << 3)
	};
	FRIEND_ENUM_CLASS_FLAGS(EPayloadFilter);

	EPayloadFilter Filter = EPayloadFilter::None;
};
