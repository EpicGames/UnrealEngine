// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrecachePayloadsCommandlet.h"

#include "CommandletUtils.h"
#include "UObject/PackageTrailer.h"
#include "Virtualization/VirtualizationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrecachePayloadsCommandlet)

UPrecachePayloadsCommandlet::UPrecachePayloadsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UPrecachePayloadsCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPrecachePayloadsCommandlet);

	TArray<FString> PackageNames = UE::Virtualization::DiscoverPackages(Params, UE::Virtualization::EFindPackageFlags::ExcludeEngineContent);

	UE_LOGF(LogVirtualization, Display, "Found %d packages", PackageNames.Num());

	TArray<FIoHash> PayloadIds = UE::Virtualization::FindVirtualizedPayloads(PackageNames);

	if (PayloadIds.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Display, "No virtualized payloads found");
		return 0;
	}

	UE_LOGF(LogVirtualization, Display, "Found %d virtualized payloads to precache", PayloadIds.Num());
	UE_LOGF(LogVirtualization, Display, "Precaching payloads...");
	
	const int32 BatchSize = 64;
	UE::Virtualization::PullPayloadsThreaded(PayloadIds, BatchSize, TEXT("Cached"), [](const UE::Virtualization::FPullRequest& Request)
		{
			if (!Request.IsSuccess())
			{
				UE_LOGF(LogVirtualization, Error, "%ls: Failed to precache payload", *LexToString(Request.GetIdentifier()));
			}
		});

	UE_LOGF(LogVirtualization, Display, "Precaching complete!");

	UE::Virtualization::IVirtualizationSystem::Get().DumpStats();
	
	return  0;
}
