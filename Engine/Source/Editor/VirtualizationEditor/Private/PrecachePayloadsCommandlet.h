// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "UObject/ObjectMacros.h"

#include "PrecachePayloadsCommandlet.generated.h"

/**
 * Attempts to pull all virtualized content in the current project. Since the virtualization
 * system will push any payloads pulled from persistent storage to cached storage this 
 * has the side effect of making sure that all payloads are stored in cached storage. This
 * can be used to reduce the cost of virtualized payload access for a user at the cost of
 * disk space.
 *
 *
 * Because the commandlet is the VirtualizationEditor module it needs to be invoked
 * with the command line:
 * -run="VirtualizationEditor.PrecachePayloads"
 */
UCLASS()
class UPrecachePayloadsCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);
};
