// Copyright Epic Games, Inc. All Rights Reserved.

#include "RehydrateProjectCommandlet.h"

#include "CommandletUtils.h"
#include "Virtualization/VirtualizationSystem.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RehydrateProjectCommandlet)

URehydrateProjectCommandlet::URehydrateProjectCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

int32 URehydrateProjectCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URehydrateProjectCommandlet);

	if (!ParseCmdline(Params))
	{
		UE_LOGF(LogVirtualization, Error, "Failed to parse the command line correctly");
		return -1;
	}

	UE_LOGF(LogVirtualization, Display, "Scanning for project packages...");
	TArray<FString> Packages = UE::Virtualization::DiscoverPackages(Params, UE::Virtualization::EFindPackageFlags::ExcludeEngineContent);

	UE_LOGF(LogVirtualization, Display, "Rehydrating packages...");

	UE::Virtualization::IVirtualizationSystem& System = UE::Virtualization::IVirtualizationSystem::Get();
	UE::Virtualization::FRehydrationResult Result = System.TryRehydratePackages(Packages, UE::Virtualization::ERehydrationOptions::Checkout);
	if (Result.WasSuccessful())
	{
		UE_LOGF(LogVirtualization, Display, "Rehydrated %d package(s)", Result.RehydratedPackages.Num());
		if (!Result.CheckedOutPackages.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Display, "Checked out %d package(s) from revision control", Result.CheckedOutPackages.Num());
		}

		if (bAutoCheckIn && !Result.CheckedOutPackages.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Display, "Attempting to check in modified packages to revision control...");

			TArray<FString> CheckedOutPackages = SourceControlHelpers::PackageFilenames(Result.CheckedOutPackages);

			const FText FinalDescription = FText::FromString(TEXT("Rehydrating project packages - automated submit from the RehydrateProject commandlet"));

			TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
			CheckInOperation->SetDescription(FinalDescription);

			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			if (SourceControlProvider.Execute(CheckInOperation, CheckedOutPackages) == ECommandResult::Succeeded)
			{
				UE_LOGF(LogVirtualization, Display, "Success! - %ls", *CheckInOperation->GetSuccessMessage().ToString());
			}
			else
			{
				// The FCheckIn operation tends to log error messages so we don't need to extract the result info
				// from the operation.
				UE_LOGF(LogVirtualization, Error, "Failed to check in the package(s) see the above LogSourceControl errors");
			}
		}

		return 0;
	}
	else
	{
		return -1;
	}
}

bool URehydrateProjectCommandlet::ParseCmdline(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	bAutoCheckIn = Switches.Contains(TEXT("AutoCheckIn"));

	return true;
}
