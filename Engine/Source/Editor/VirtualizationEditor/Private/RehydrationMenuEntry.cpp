// Copyright Epic Games, Inc. All Rights Reserved.

#include "RehydrationMenuEntry.h"

#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "VirtualizationEditor"

namespace UE::Virtualization
{

/**
 * Attempt to rehydrate the provided files although this currently assumes that the user has
 * checked the files out of source control before hand.
 */
void RehydratePackages(TArray<FString> SelectedFiles)
{
	//TArray<FText> Errors;

	FRehydrationResult Result = IVirtualizationSystem::Get().TryRehydratePackages(SelectedFiles, ERehydrationOptions::Checkout);
	if (Result.WasSuccessful())
	{
		// TODO: At some point ::TryRehydratePackages will return more detail info about the process
		// when it does we should make a better job at logging it.

		const FText Message = LOCTEXT("RehydrationSucccess", "Files were successfully re-hydrated");

		FNotificationInfo Info(Message);
		Info.bFireAndForget = true;
		Info.ExpireDuration = 2.0f;

		FSlateNotificationManager::Get().AddNotification(Info);
	}
	else
	{
		const bool bForceNotification = true;

		FMessageLog Log("LogVirtualization");

		/*for (const FText& Error : Errors)
		{
			Log.Error(Error);
		}*/

		Log.Notify(LOCTEXT("RehydrationFailed", "Failed to rehydrate packages, see the message log for more info"), EMessageSeverity::Info, bForceNotification);
	}
}

void BrowserItemsToFilePaths(const TArray<FContentBrowserItem>& Source, TArray<FString>& OutResult)
{
	for (const FContentBrowserItem& SelectedItem : Source)
	{
		const FContentBrowserItemData* SelectedItemData = SelectedItem.GetPrimaryInternalItem();
		if (SelectedItemData == nullptr)
		{
			continue;
		}

		if (!SelectedItemData->IsFile())
		{
			continue;
		}

		UContentBrowserDataSource* DataSource = SelectedItemData->GetOwnerDataSource();
		if (DataSource == nullptr)
		{
			continue;
		}

		FString Path;
		if (DataSource->GetItemPhysicalPath(*SelectedItemData, Path))
		{
			OutResult.Add(FPaths::ConvertRelativePathToFull(Path));
		}
	}
}

void SetupRehydrationContentMenuEntry()
{
	UToolMenus* ToolsMenu = UToolMenus::Get();
	if (ToolsMenu == nullptr)
	{
		return;
	}

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.AssetActionsSubMenu");
	if (Menu == nullptr)
	{
		return;
	}

	Menu->AddDynamicSection("VirtualizedAssetsDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			const UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();

			if (Context == nullptr)
			{
				return;
			}

			if (Context->bCanBeModified == false)
			{
				return;
			}

			// The user needs to enable this experimental editor setting before we display the
			// option in the context menu. 
			if (GetDefault<UEditorExperimentalSettings>()->bVirtualizedAssetRehydration == false)
			{
				return;
			}

			TArray<FString> SelectedFiles;
			BrowserItemsToFilePaths(Context->SelectedItems, SelectedFiles);

			if (SelectedFiles.IsEmpty())
			{
				return;
			}

			FToolMenuSection& Section = Menu->AddSection("VirtualizedAssets", LOCTEXT("VirtualizedAssetsHeading", "Virtualized Assets"));
			{
				Section.AddMenuEntry(
					"RehydrateAsset",
					LOCTEXT("RehydrateAsset", "Rehydrate Asset"),
					LOCTEXT("RehydrateAssetTooltip", "Pulls the assets virtualized payloads and stores them in the package file once more"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateStatic(&RehydratePackages, SelectedFiles))
				);
			}
		}));
}

} // namespace UE::Virtualization

#undef LOCTEXT_NAMESPACE
