// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::Virtualization
{

/** Adds an option to rehydrate packages under the asset context menu */
void SetupRehydrationContentMenuEntry();

} // namespace UE::Virtualization
