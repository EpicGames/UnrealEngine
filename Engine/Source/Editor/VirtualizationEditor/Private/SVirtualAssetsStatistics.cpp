// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVirtualAssetsStatistics.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "FrameNumberTimeEvaluator.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/Attribute.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "SlotBase.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/UObjectGlobals.h"
#include "Virtualization/VirtualizationTypes.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "VirtualizationEditor"

namespace
{

static FString SingleDecimalFormat(double Value)
{
	const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(true)
		.SetMinimumFractionalDigits(1)
		.SetMaximumFractionalDigits(1);
	return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
}

} //namespace

SVirtualAssetsStatisticsDialog::SVirtualAssetsStatisticsDialog()
{
	using namespace UE::Virtualization;
	
	// TODO - need a way to make this work once the system is initialized
	// Register our VA notification delegate with the event
	if (IVirtualizationSystem::IsInitialized())
	{
		IVirtualizationSystem& System = IVirtualizationSystem::Get();
		System.GetNotificationEvent().AddRaw(this, &SVirtualAssetsStatisticsDialog::OnNotificationEvent);
	}
}

SVirtualAssetsStatisticsDialog::~SVirtualAssetsStatisticsDialog()
{
	using namespace UE::Virtualization;

	// Unregister our VA notification delegate with the event
	IVirtualizationSystem& System = IVirtualizationSystem::Get();
	System.GetNotificationEvent().RemoveAll(this);
}

void SVirtualAssetsStatisticsDialog::OnNotificationEvent(UE::Virtualization::IVirtualizationSystem::ENotification Notification, const FIoHash& PayloadId)
{
	using namespace UE::Virtualization;

	FScopeLock SocpeLock(&NotificationCS);
	
	switch (Notification)
	{	
		case IVirtualizationSystem::ENotification::PullBegunNotification:
		{
			IsPulling = true;
			NumPullRequests++;

			break;
		}

		case IVirtualizationSystem::ENotification::PullEndedNotification:
		{	
			if (IsPulling == true)
			{
				NumPullRequests--;
				IsPulling = NumPullRequests!=0;
			}
			
			break;
		}

		case IVirtualizationSystem::ENotification::PullFailedNotification:
		{
			NumPullRequestFailures++;
			break;
		}

		default:
		break;
	}
}

void SVirtualAssetsStatisticsDialog::Construct(const FArguments& InArgs)
{
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 20, 0, 0)
		.Expose(GridSlot)
		[
			SAssignNew(ScrollBox, SScrollBox)
			.Orientation(EOrientation::Orient_Horizontal)
			.ScrollBarAlwaysVisible(false)

			+ SScrollBox::Slot()
			[
				GetGridPanel()
			]	
		]
	];

	RegisterActiveTimer(0.25f, FWidgetActiveTimerDelegate::CreateSP(this, &SVirtualAssetsStatisticsDialog::UpdateGridPanels));
}

EActiveTimerReturnType SVirtualAssetsStatisticsDialog::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	ScrollBox->ClearChildren();
	ScrollBox->AddSlot()
		[
			GetGridPanel()
		];
		
	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	const float PullNotifactionTimeLimit=1.0f;

	// Only show the pull notification if we have been pulling for more than a second..
	if (NumPullRequests != 0)
	{
		PullNotificationTimer += InDeltaTime;
	}
	else
	{
		PullNotificationTimer = 0.0f;
	}

	if ( PullNotificationTimer>PullNotifactionTimeLimit && PullRequestNotificationItem.IsValid()==false )
	{
		// No existing notification or the existing one has finished
		FNotificationInfo Info(LOCTEXT("PayloadSyncNotifcation", "Syncing Asset Payloads"));
		Info.bFireAndForget = false;
		Info.bUseLargeFont = false;
		Info.bUseThrobber = false;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 0.0f;

		PullRequestNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		if (PullRequestNotificationItem.IsValid())
		{
			PullRequestNotificationItem->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}

	if ( NumPullRequestFailures>0 && PullRequestFailedNotificationItem.IsValid()==false )
	{
		// No existing notification or the existing one has finished
		FNotificationInfo Info(LOCTEXT("PayloadFailedNotifcation", "Failed to sync some Virtual Asset payloads from available backends.\nSome assets may no longer be usable.."));	
		Info.bFireAndForget = false;
		Info.bUseLargeFont = false;
		Info.bUseThrobber = false;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 0.0f;
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("PullFailedIgnore", "Ignore"), LOCTEXT("PullFailedIgnoreToolTip", "Ignore future warnings"), FSimpleDelegate::CreateSP(this, &SVirtualAssetsStatisticsDialog::OnWarningReasonIgnore), SNotificationItem::CS_None));
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("PullFailedOK", "Ok"), LOCTEXT("PullFailedOkToolTip", "Notify future warnings"), FSimpleDelegate::CreateSP(this, &SVirtualAssetsStatisticsDialog::OnWarningReasonOk), SNotificationItem::CS_None));
		Info.HyperlinkText = LOCTEXT("PullFailed_ShowLog", "Show Message Log");
		Info.Hyperlink = FSimpleDelegate::CreateStatic([]() { FMessageLog("LogVirtualization").Open(EMessageSeverity::Warning, true); });

		PullRequestFailedNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
	}
	
	if ( NumPullRequests==0 && PullRequestNotificationItem.IsValid()==true )
	{
		PullRequestNotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		PullRequestNotificationItem->ExpireAndFadeout();
		PullRequestNotificationItem.Reset();
	}

	return EActiveTimerReturnType::Continue;
}

void SVirtualAssetsStatisticsDialog::OnWarningReasonOk()
{
	if (PullRequestFailedNotificationItem.IsValid() == true)
	{
		PullRequestFailedNotificationItem->ExpireAndFadeout();
		PullRequestFailedNotificationItem.Reset();
		NumPullRequestFailures = 0;
	}
}

void SVirtualAssetsStatisticsDialog::OnWarningReasonIgnore()
{
	if (PullRequestFailedNotificationItem.IsValid() == true)
	{
		PullRequestFailedNotificationItem->ExpireAndFadeout();
	}
}

TSharedRef<SWidget> SVirtualAssetsStatisticsDialog::GetGridPanel()
{
	using namespace UE::Virtualization;

	const float RowMargin = 0.0f;
	const float TitleMargin = 10.0f;
	const float ColumnMargin = 10.0f;
	const float BorderPadding = ColumnMargin / 2.0f;

	const FMargin StdMargin(ColumnMargin, RowMargin);
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	
	FSlateColor Color = FStyleColors::Foreground;
	FSlateFontInfo Font = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	struct FPanels
	{
		TSharedPtr<SGridPanel> Names;
		TSharedPtr<SGridPanel> Pull;
		TSharedPtr<SGridPanel> Cache;
		TSharedPtr<SGridPanel> Push;

	} Panels;

	// Early out if the system is disabled
	if (IVirtualizationSystem::Get().IsEnabled() == false)
	{
		return	SNew(STextBlock)
				.Margin(FMargin(ColumnMargin, RowMargin))
				.ColorAndOpacity(TitleColor)
				.Font(TitleFont)
				.Justification(ETextJustify::Center)
				.Text(LOCTEXT("Disabled", "Virtual Assets Are Disabled For This Project"));
	}

	TSharedRef<SHorizontalBox> Panel = SNew(SHorizontalBox);
	Panel->AddSlot()
	.Padding(BorderPadding)
	.AutoWidth()
	[
		SAssignNew(Panels.Names, SGridPanel)
		+ SGridPanel::Slot(0, 0)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin + (BorderPadding / 2.0f), RowMargin + (BorderPadding / 2.0f)))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Justification(ETextJustify::Left)
		]
		+SGridPanel::Slot(0, 1)
		[
			SNew(STextBlock)
			.Margin(FMargin(ColumnMargin, RowMargin, 0.0f, TitleMargin))
			.ColorAndOpacity(TitleColor)
			.Font(TitleFont)
			.Justification(ETextJustify::Left)
			.Text(LOCTEXT("Backend", "Backend"))
		]
	];

	auto CreateGridPanels = [&](FText&& Label)
		{
			TSharedPtr<SGridPanel> GridPanel;

			Panel->AddSlot()
			.Padding(BorderPadding)
			.AutoWidth()
			[
				SNew(SBorder)
				.Padding(BorderPadding)
				[
					SAssignNew(GridPanel, SGridPanel)
					+ SGridPanel::Slot(1, 0)
					[
						SNew(STextBlock)
						.Margin(FMargin(ColumnMargin, RowMargin))
						.ColorAndOpacity(TitleColor)
						.Font(TitleFont)
						.Justification(ETextJustify::Center)
						.Text(MoveTemp(Label))
					]
					+ SGridPanel::Slot(0, 1)
					[
						SNew(STextBlock)
						.Margin(FMargin(ColumnMargin, RowMargin, 0.0f, TitleMargin))
						.ColorAndOpacity(TitleColor)
						.Font(TitleFont)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("Count", "Count"))
					]
					+ SGridPanel::Slot(1, 1)
					[
						SNew(STextBlock)
						.Margin(FMargin(ColumnMargin, RowMargin, 0.0f, TitleMargin))
						.ColorAndOpacity(TitleColor)
						.Font(TitleFont)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("Size", "Size (MiB)"))
					]
					+ SGridPanel::Slot(2, 1)
					[
						SNew(STextBlock)
						.Margin(FMargin(ColumnMargin, RowMargin, 0.0f, TitleMargin))
						.ColorAndOpacity(TitleColor)
						.Font(TitleFont)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("Time", "Avg (ms)"))
					]
				]
			];
			
			return GridPanel;
		};

	Panels.Pull = CreateGridPanels(LOCTEXT("Download", "Download"));
	Panels.Cache = CreateGridPanels(LOCTEXT("Cache", "Cache"));
	Panels.Push = CreateGridPanels(LOCTEXT("Upload", "Upload"));

	int32 RowIndex = 2;
	auto DisplayPayloadActivityInfo = [&StdMargin, &Color, &Font, &RowIndex, &Panels](const FString& Name, const FPayloadActivityInfo& PayloadActivityInfo)
		{
			Panels.Names->AddSlot(0, RowIndex)
			[
				SNew(STextBlock)
				.Margin(StdMargin)
				.ColorAndOpacity(Color)
				.Font(Font)
				.Justification(ETextJustify::Left)
				.Text(FText::FromString(Name))
			];

			auto FillPanelDetails = [&StdMargin, &Color, &Font, &RowIndex](TSharedPtr<SGridPanel>& Panel, const FPayloadActivityInfo::FActivity& Activity)
				{
					Panel->AddSlot(0, RowIndex)
						[
							SNew(STextBlock)
							.Margin(StdMargin)
							.ColorAndOpacity(Color)
							.Font(Font)
							.Justification(ETextJustify::Center)
							.Text(FText::FromString(FString::Printf(TEXT("%" INT64_FMT), Activity.PayloadCount)))
						];

					const double TotalBytesMiB = static_cast<double>(Activity.TotalBytes) / (1024.0 * 1024.0);

					Panel->AddSlot(1, RowIndex)
						[
							SNew(STextBlock)
							.Margin(StdMargin)
							.ColorAndOpacity(Color)
							.Font(Font)
							.Justification(ETextJustify::Center)
							.Text(FText::FromString(SingleDecimalFormat(TotalBytesMiB)))
						];

					const double TotalTime = static_cast<double>(FPlatformTime::ToMilliseconds64(Activity.CyclesSpent));
					const double Avg = Activity.PayloadCount > 0 ? TotalTime / static_cast<double>(Activity.PayloadCount) : 0.0;

					Panel->AddSlot(2, RowIndex)
						[
							SNew(STextBlock)
							.Margin(StdMargin)
							.ColorAndOpacity(Color)
							.Font(Font)
							.Justification(ETextJustify::Center)
							.Text(FText::FromString(SingleDecimalFormat(Avg)))
						];
				};

			FillPanelDetails(Panels.Pull, PayloadActivityInfo.Pull);
			FillPanelDetails(Panels.Cache, PayloadActivityInfo.Cache);
			FillPanelDetails(Panels.Push, PayloadActivityInfo.Push);

			RowIndex++;
		};

	TArray<FBackendStats> BackendStats = IVirtualizationSystem::Get().GetBackendStatistics();
	for (const FBackendStats& Stats : BackendStats)
	{
		DisplayPayloadActivityInfo(Stats.ConfigName, Stats.PayloadActivity);
	}

	FPayloadActivityInfo AccumulatedPayloadAcitvityInfo = IVirtualizationSystem::Get().GetSystemStatistics();

	Color = TitleColor;
	Font = TitleFont;

	DisplayPayloadActivityInfo(FString("Total"), AccumulatedPayloadAcitvityInfo);

	return Panel;
}

#undef LOCTEXT_NAMESPACE
