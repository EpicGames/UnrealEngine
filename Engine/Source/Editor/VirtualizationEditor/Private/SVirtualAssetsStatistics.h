// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "IO/IoHash.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

class SNotificationItem;
class SWidget;

class SVirtualAssetsStatisticsDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SVirtualAssetsStatisticsDialog) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	SVirtualAssetsStatisticsDialog();
	virtual ~SVirtualAssetsStatisticsDialog();

private:

	TSharedRef<SWidget> GetGridPanel();

	EActiveTimerReturnType UpdateGridPanels(double InCurrentTime, float InDeltaTime);

	FText GetNotificationText() const;

	void OnNotificationEvent(UE::Virtualization::IVirtualizationSystem::ENotification Notification, const FIoHash& PayloadId);
	void OnWarningReasonOk();
	void OnWarningReasonIgnore();

	SVerticalBox::FSlot* GridSlot = nullptr;

	FCriticalSection NotificationCS;

	TSharedPtr<SNotificationItem> PullRequestNotificationItem;
	TSharedPtr<SNotificationItem> PullRequestFailedNotificationItem;

	TSharedPtr<class SScrollBox> ScrollBox;

	bool	IsPulling = false;
	float	PullNotificationTimer = 0.0f;
	uint32	NumPullRequests = 0;
	uint32	NumPullRequestFailures = 0;
};

