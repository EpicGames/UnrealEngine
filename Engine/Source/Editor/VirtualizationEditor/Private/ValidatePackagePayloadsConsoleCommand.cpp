// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleCommandUtils.h"

#include "CommandletUtils.h"
#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "UObject/PackageResourceManager.h"
#include "Virtualization/VirtualizationSystem.h"

namespace UE::Virtualization
{

bool ValidatePayload(FIoHash Id, FCompressedBuffer CompressedPayload)
{
	if (CompressedPayload.IsNull())
	{
		UE_LOGF(LogVirtualization, Error, "\tCould not load payload '%ls'", *LexToString(Id));
		return false;
	}

	if (CompressedPayload.GetRawHash() != Id)
	{
		UE_LOGF(LogVirtualization, Error, "\tPayload '%ls' had the wrong hash (%ls) after loading", *LexToString(Id), *LexToString(CompressedPayload.GetRawHash()));
		return false;
	}

	FSharedBuffer UncompressedPayload = CompressedPayload.Decompress();

	if (UncompressedPayload.IsNull())
	{
		UE_LOGF(LogVirtualization, Error, "\tCould not decompress payload '%ls'", *LexToString(Id));
		return false;
	}

	FIoHash UncompressedHash = FIoHash::HashBuffer(UncompressedPayload);
	if (UncompressedHash != Id)
	{
		UE_LOGF(LogVirtualization, Error, "\tPayload '%ls' had the wrong hash (%ls) after decompression", *LexToString(Id), *LexToString(UncompressedHash));
		return false;
	}

	return true;
}
void ValidatePackagePayloads(const TArray<FString>& Args)
{
	if (Args.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Error, "Command 'ValidatePackagePayloads' called without any arguments");
		return;
	}

	TArray<TPair<FString, UE::FPackageTrailer>> Packages = LoadPackageTrailerFromArgs(Args);

	if (Packages.IsEmpty())
	{
		UE_LOGF(LogVirtualization, Error, "Command 'ValidatePackagePayloads' failed to find any valid packages from the provided arguments");
		return;
	}

	for (const TPair<FString, UE::FPackageTrailer>& Package : Packages)
	{
		const FString& PackagePath = Package.Key;
		const UE::FPackageTrailer& Trailer = Package.Value;

		UE_LOGF(LogVirtualization, Display, "Validating package '%ls'", *PackagePath);

		int32 ValidLocalPayloads = 0;
		TArray<FIoHash> LocalPayloads = Trailer.GetPayloads(EPayloadStorageType::Local);

		if (!LocalPayloads.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Display, "Checking Local Payloads...");

			TUniquePtr<FArchive> Ar = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, Package.Key);
			check(Ar.IsValid());

			for (const FIoHash& Id : LocalPayloads)
			{
				FCompressedBuffer CompressedPayload = Trailer.LoadLocalPayload(Id, *Ar);

				if (ValidatePayload(Id, CompressedPayload))
				{
					ValidLocalPayloads++;
				}
			}
		}

		int32 ValidVirtualizedPayloads = 0;
		TArray<FIoHash> VirtualizedPayloads = Trailer.GetPayloads(EPayloadStorageType::Virtualized);

		if (!VirtualizedPayloads.IsEmpty())
		{
			UE_LOGF(LogVirtualization, Display, "Checking Virtualized Payloads...");

			TArray<FPullRequest> Requests = ToRequestArray(VirtualizedPayloads);
			IVirtualizationSystem::Get().PullData(Requests);

			check(Requests.Num() == VirtualizedPayloads.Num());

			for(int32 Index = 0; Index < VirtualizedPayloads.Num(); ++Index)
			{
				const FPullRequest& Request = Requests[Index];

				if (ValidatePayload(Request.GetIdentifier(), Request.GetPayload()))
				{
					ValidLocalPayloads++;
				}
			}
		}

		UE_CLOGF(!LocalPayloads.IsEmpty(), LogVirtualization, Display, "%d/%d local payloads are valid", ValidLocalPayloads, LocalPayloads.Num());
		UE_CLOGF(!VirtualizedPayloads.IsEmpty(), LogVirtualization, Display, "%d/%d virtualized payloads are valid", ValidVirtualizedPayloads, VirtualizedPayloads.Num());
	}
}

static FAutoConsoleCommand CCmdValidatePackagePayloads = FAutoConsoleCommand(
	TEXT("ValidatePackagePayloads"),
	TEXT("Checks all payloads that a package references and makes sure that they are valid"),
	FConsoleCommandWithArgsDelegate::CreateStatic(ValidatePackagePayloads));
} //namespace UE::Virtualization
