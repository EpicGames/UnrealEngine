// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValidateVirtualizedContentCommandlet.h"

#include "CommandletUtils.h"
#include "UObject/PackageTrailer.h"
#include "Virtualization/VirtualizationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ValidateVirtualizedContentCommandlet)

namespace UE
{

/** Utility class so we can find all of the packages that use a given payload id */
class FIoHashToPackagesLookup
{
public:
	FIoHashToPackagesLookup(const TMap<FString, UE::FPackageTrailer>& InPackages)
	{
		PackagePaths.SetNum(InPackages.Num());

		for (const TPair<FString, UE::FPackageTrailer>& Package : InPackages)
		{
			const int32 Index = PackagePaths.Num();
			PackagePaths.Add(Package.Key);

			const TArray<FIoHash> VirtualizedPayloads = Package.Value.GetPayloads(UE::EPayloadStorageType::Virtualized);
			for (const FIoHash& Id : VirtualizedPayloads)
			{
				PackageLookup.FindOrAdd(Id).Add(Index);
			}
		}
	}

	void PrintError(const FIoHash& Id, const TCHAR* ErrorMessage) const
	{
		TStringBuilder<1028> FinalMessage;
		FinalMessage << TEXT("Payload ") << Id << TEXT(" - ") << ErrorMessage;

		for (int32 PackageIndex : PackageLookup[Id])
		{
			FinalMessage << LINE_TERMINATOR << TEXT(" \t") << PackagePaths[PackageIndex];
		}

		UE_LOGF(LogVirtualization, Error, "%ls", FinalMessage.ToString());
	}

private:
	TArray<FString> PackagePaths;
	TMap<FIoHash, TArray<uint32, TInlineAllocator<4>>> PackageLookup;
};

} //namespace UE

UValidateVirtualizedContentCommandlet::UValidateVirtualizedContentCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


int32 UValidateVirtualizedContentCommandlet::Main(const FString& Params)
{
	using namespace UE::Virtualization;

	TRACE_CPUPROFILER_EVENT_SCOPE(UValidateVirtualizedContentCommandlet);

	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	const bool bValidateContent = Switches.Contains(TEXT("ValidateContent"));

	UE_LOGF(LogVirtualization, Display, "Finding packages in the project...");
	TArray<FString> PackagePaths = FindPackages(EFindPackageFlags::ExcludeEngineContent);
	UE_LOGF(LogVirtualization, Display, "Found %d package(s)", PackagePaths.Num());

	TMap<FString, UE::FPackageTrailer> Packages;
	TSet<FIoHash> Payloads;

	UE_LOGF(LogVirtualization, Display, "Scanning package(s) for virtualized payloads...");
	FindVirtualizedPayloadsAndTrailers(PackagePaths, Packages, Payloads);
	UE_LOGF(LogVirtualization, Display, "Found %d virtualized package(s) with %d unique payload(s)", Packages.Num(), Payloads.Num());

	UE::FIoHashToPackagesLookup PkgLookupTable(Packages);

	int32 ErrorCount = 0;
	if (bValidateContent)
	{
		ErrorCount = ValidatePayloadContent(Packages, Payloads, PkgLookupTable);
	}
	else
	{
		ErrorCount = ValidatePayloadsExists(Packages, Payloads);
	}

	if (ErrorCount == 0)
	{
		UE_LOGF(LogVirtualization, Display, "All virtualized payloads could be found in persistent storage");
		return 0;
	}
	else
	{
		UE_LOGF(LogVirtualization, Error, "%d/%d package(s) had at least one virtualized payload missing from persistent storage", ErrorCount, Packages.Num());
		return 0;
	}
}

int32 UValidateVirtualizedContentCommandlet::ValidatePayloadsExists(const TMap<FString, UE::FPackageTrailer>& Packages, const TSet<FIoHash>& Payloads)
{
	using namespace UE::Virtualization;

	UE_LOGF(LogVirtualization, Display, "Validating payloads existence..");

	IVirtualizationSystem& System = IVirtualizationSystem::Get();

	UE_LOGF(LogVirtualization, Display, "Querying the state of the virtualized payload(s) in persistent storage...");
	TArray<EPayloadStatus> PayloadStatuses;
	if (System.QueryPayloadStatuses(Payloads.Array(), EStorageType::Persistent, PayloadStatuses) != EQueryResult::Success)
	{
		UE_LOGF(LogVirtualization, Error, "Failed to query the statuses of the payload(s)");
		return 1;
	}

	int32 ErrorCount = 0;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ValidatePackages);

		UE_LOGF(LogVirtualization, Display, "Checking for missing payloads...");

		for (const TPair<FString, UE::FPackageTrailer>& Package : Packages)
		{
			bool bFoundErrors = false;

			const UE::FPackageTrailer& Trailer = Package.Value;
			const TArray<FIoHash> VirtualizedPayloads = Trailer.GetPayloads(UE::EPayloadStorageType::Virtualized);

			for (const FIoHash& PayloadId : VirtualizedPayloads)
			{
				const int32 Index = Payloads.FindId(PayloadId).AsInteger();
				if (PayloadStatuses[Index] == EPayloadStatus::FoundPartial)
				{
					// TODO: We currently don't have a way to inform the user which persistent backend the payload is not in!
					UE_LOGF(LogVirtualization, Error, "%ls: Payload '%ls' could not be found in all persistent backends", *Package.Key, *LexToString(PayloadId));
					bFoundErrors = true;
				}
				else if (PayloadStatuses[Index] != EPayloadStatus::FoundAll)
				{
					UE_LOGF(LogVirtualization, Error, "%ls: Payload '%ls' could not be found in any persistent backend", *Package.Key, *LexToString(PayloadId));
					bFoundErrors = true;
				}
			}

			if (bFoundErrors)
			{
				ErrorCount++;
			}
		}
	}

	return ErrorCount;
}

int32 UValidateVirtualizedContentCommandlet::ValidatePayloadContent(const TMap<FString, UE::FPackageTrailer>& Packages, const TSet<FIoHash>& Payloads, const UE::FIoHashToPackagesLookup& PkgLookupTable)
{
	UE_LOGF(LogVirtualization, Display, "Validating payloads existence and content...");

	std::atomic<int32> ErrorCount = 0;

	const int32 BatchSize = 64;
	UE::Virtualization::PullPayloadsThreaded(Payloads.Array(), BatchSize, TEXT("Validated"), [&PkgLookupTable , &ErrorCount](const UE::Virtualization::FPullRequest& Request)
		{
			if (!Request.IsSuccess() || Request.GetPayload().IsNull())
			{
				PkgLookupTable.PrintError(Request.GetIdentifier(), TEXT("could not be pulled"));

				ErrorCount++;
				return;
			}
			
			if (Request.GetPayload().GetRawHash() != Request.GetIdentifier())
			{
				PkgLookupTable.PrintError(Request.GetIdentifier(), TEXT("pulled a payload with the wrong hash"));

				ErrorCount++;
				return;
			}

			FSharedBuffer UncompressedPayload = Request.GetPayload().Decompress();
			if (UncompressedPayload.IsNull())
			{
				PkgLookupTable.PrintError(Request.GetIdentifier(), TEXT("could not be decompressed"));

				ErrorCount++;
				return;
			}

			if (FIoHash::HashBuffer(UncompressedPayload) != Request.GetIdentifier())
			{
				PkgLookupTable.PrintError(Request.GetIdentifier(), TEXT("is corrupted"));

				ErrorCount++;
				return;
			}
		});

	return ErrorCount;
}

