// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "UObject/ObjectMacros.h"

#include "ValidateVirtualizedContentCommandlet.generated.h"

namespace UE { class FPackageTrailer; }
namespace UE { class FIoHashToPackagesLookup; }

class FIoHashToPackagesLookup;
struct FIoHash;

/**
 * Iterates over all of the packages in a project and identifies which packages contain
 * references to virtualized payloads. The commandlet will then check that all virtualized
 * payloads can be found in persistent storage. Error messages will be logged for
 * packages that contain virtualized payloads that cannot be found in one or more persistent
 * storage backend.
 * 
 * Because the commandlet is the VirtualizationEditor module it needs to be invoked
 * with the command line:
 * -run="VirtualizationEditor.ValidateVirtualizedContent"
 * 
 * Optional Args:
 * bValidateContent:	When this switch is set each payload will be pulled from persistent storage
 *						and additional validations will be run to make sure that the payload is
 *						not corrupted. This is much slower than just checking that the payload
 *						exists.
 */
UCLASS()
class UValidateVirtualizedContentCommandlet
	: public UCommandlet
{
private:
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);

	int32 ValidatePayloadsExists(const TMap<FString, UE::FPackageTrailer>& Packages, const TSet<FIoHash>& Payloads);

	int32 ValidatePayloadContent(const TMap<FString, UE::FPackageTrailer>& Packages, const TSet<FIoHash>& Payloads, const UE::FIoHashToPackagesLookup& PkgLookupTable);
};
