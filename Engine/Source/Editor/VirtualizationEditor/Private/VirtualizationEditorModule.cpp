// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationEditorModule.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleManager.h"
#include "RehydrationMenuEntry.h"
#include "SVirtualAssetsStatistics.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "VirtualizationEditor"

IMPLEMENT_MODULE(FVirtualizationEditorModule, VirtualizationEditor);

static const FName VirtualAssetsStatisticsTabName = FName(TEXT("VirtualAssetsStatistics"));

namespace 
{

/** Utility function for adding menu entries, called when the module is started */
void SetupMenuEntries()
{
	UE::Virtualization::SetupRehydrationContentMenuEntry();
}

} //namespace

void FVirtualizationEditorModule::StartupModule()
{
	const FSlateIcon VirtaulAssetsStatisticsIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Statistics");
	
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(VirtualAssetsStatisticsTabName, FOnSpawnTab::CreateRaw(this, &FVirtualizationEditorModule::CreateVirtualAssetsStatisticsTab))
		.SetDisplayName(LOCTEXT("VirtualAssetsStatisticsTabTitle", "Virtual Assets"))
		.SetTooltipText(LOCTEXT("VirtualAssetsStatisticsTabToolTipText", "Virtual Assets  Statistics"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsAuditCategory())
		.SetIcon(VirtaulAssetsStatisticsIcon);

#if WITH_RELOAD
	// This code attempts to relaunch the tabs when you reload this module
	if (IsReloadActive() && FSlateApplication::IsInitialized())
	{
		ShowVirtualAssetsStatisticsTab();
	}
#endif // WITH_RELOAD

	SetupMenuEntries();
}

void FVirtualizationEditorModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		if (VirtualAssetsStatisticsTab.IsValid())
		{
			VirtualAssetsStatisticsTab.Pin()->RequestCloseTab();
		}

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(VirtualAssetsStatisticsTabName);
	}
}

void FVirtualizationEditorModule::ShowVirtualAssetsStatisticsTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(VirtualAssetsStatisticsTabName));
}

TSharedPtr<SWidget> FVirtualizationEditorModule::CreateVirtualAssetsStatisticsDialog()
{
	return SNew(SVirtualAssetsStatisticsDialog);
}

TSharedRef<SDockTab> FVirtualizationEditorModule::CreateVirtualAssetsStatisticsTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(VirtualAssetsStatisticsTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateVirtualAssetsStatisticsDialog().ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE
