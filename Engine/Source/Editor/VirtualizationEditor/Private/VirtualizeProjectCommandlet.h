// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "UObject/ObjectMacros.h"

#include "VirtualizeProjectCommandlet.generated.h"

/**
 * Finds all packages in the project and attempts to virtualize their content. If revision control
 * is enabled then the commandlet will attempt to checkout the packages that need modification.
 *
 * Because the commandlet is the VirtualizationEditor module it needs to be invoked
 * with the command line:
 * -run="VirtualizationEditor.VirtualizeProject"
 */
UCLASS()
class UVirtualizeProjectCommandlet
	: public UCommandlet
{
private:

	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);

	bool ParseCmdline(const FString& Params);

private:

	bool bAutoCheckIn = false;
};
