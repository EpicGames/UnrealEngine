// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class SWidget;
class SDockTab;
class FSpawnTabArgs;

/**
 * The module holding all of the UI related pieces for the Virtualization module
 */
class FVirtualizationEditorModule : public IModuleInterface
{
private:

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void ShowVirtualAssetsStatisticsTab();

	TSharedPtr<SWidget> CreateVirtualAssetsStatisticsDialog();
	TSharedRef<SDockTab> CreateVirtualAssetsStatisticsTab(const FSpawnTabArgs& Args);

	TWeakPtr<SDockTab> VirtualAssetsStatisticsTab;
};
