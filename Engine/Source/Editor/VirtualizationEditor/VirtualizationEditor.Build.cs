// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class VirtualizationEditor : ModuleRules
{
	public VirtualizationEditor(ReadOnlyTargetRules Target)
		 : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ContentBrowserData",
				"Core",
				"CoreUObject",
				"Engine",
				"UnrealEd",
				"ToolMenus",
				"SourceControl",
				"Slate",
				"SlateCore",
				"Virtualization"
			});

		PrivateIncludePathModuleNames.AddRange
		(
			new string[]
			{
				"WorkspaceMenuStructure",
			}
		);
	}
}
